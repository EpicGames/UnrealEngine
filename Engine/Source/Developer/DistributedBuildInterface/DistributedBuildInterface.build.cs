// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DistributedBuildInterface : ModuleRules
{
	public DistributedBuildInterface(ReadOnlyTargetRules Target) : base(Target)
	{
		Type = ModuleType.External;
		PublicSystemIncludePaths.Add(System.IO.Path.Combine(ModuleDirectory, "Public"));
	}
}
