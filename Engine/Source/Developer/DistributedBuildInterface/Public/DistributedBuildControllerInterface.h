// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Memory/MemoryView.h"
#include "Modules/ModuleInterface.h"
#include "Features/IModularFeatures.h"
#include "Async/Async.h"
#include "Templates/Function.h"

// Optional custom output data validator. When set on FTaskCommandData, this callback is
// invoked by HandleUbaJobFinished instead of the default format and size check. Return true
// if the output data is semantically valid for the producing task type.
using FOutputDataValidator = TFunction<bool(TArrayView<const uint8> OutputData)>;

struct FDistributedBuildTaskResult
{
	int32 ReturnCode;
	bool bCompleted;
	// If the distributed controller supports virtual files, the output file data will be populated here instead of being written to a file on disk
	TArray<uint8> OutputData;
};

struct FDistributedBuildStats
{
	uint32 MaxRemoteAgents = 0;
	uint32 MaxActiveAgentCores = 0;
};

struct FTaskCommandData
{	
	FString Command;
	FString WorkingDirectory;
	FString InputFileName;
	FString OutputFileName;
	// Command-line arguments for the remote task. Positional args first (consumed
	// by index), flags after. Controllers must APPEND their own flags, never
	// prepend, to keep the positional prefix intact.
	FString ExtraCommandArgs;
	FString Description; 									// Optional string describing the task; can be used by the controller implementation for debug purposes (i.e. UBA breadcrumbs)
	uint32 DispatcherPID = 0;
	TArray<uint8> InputFileData; 							// Optional byte array containing the contents of the input file; only used if the controller supports virtual files
	TArray <FMemoryView> SourceData; 						// Optional array of memory views containing the compressed source files; only used if the controller supports virtual files
	TSharedPtr<void, ESPMode::ThreadSafe> SourceDataOwner;	// Optional shared pointer to opaque data keeping the memory referenced by SourceData alive
	TArray<FString> AdditionalOutputFolders; 				// Optional additional folder(s) which task may write artifacts to
	FOutputDataValidator OutputValidator;					// Optional custom output data validator. If set, HandleUbaJobFinished calls this instead of the default format and size check.
};

struct FDistributedBuildTask
{
	uint32 ID;
	FTaskCommandData CommandData;
	TPromise<FDistributedBuildTaskResult> Promise;

	FDistributedBuildTask(uint32 ID, const FTaskCommandData& CommandData, TPromise<FDistributedBuildTaskResult>&& Promise)
		: ID(ID)
		, CommandData(CommandData)
		, Promise(MoveTemp(Promise))
	{
	}

	/** Sets the promised task result to being incomplete, i.e. FDistributedBuildTaskResult::bCompleted=false. */
	void Cancel()
	{
		FDistributedBuildTaskResult Result;
		Result.ReturnCode = 0;
		Result.bCompleted = false;
		Promise.SetValue(Result);
	}

	/** Sets the promised task result to being completed with the specified return code. */
	void Finalize(int32 InReturnCode, TArray<uint8>&& OutputData = {})
	{
		FDistributedBuildTaskResult Result;
		Result.ReturnCode = InReturnCode;
		Result.bCompleted = true;
		Result.OutputData = MoveTemp(OutputData);
		Promise.SetValue(Result);
	}
};

struct FTaskResponse
{
	uint32 ID;
	int32 ReturnCode;
};

class IDistributedBuildController : public IModuleInterface, public IModularFeature
{
public:
	virtual bool SupportsDynamicReloading() override { return false; }

	// Returns true if this distributed controller also supports local workers alongside remote workers. By default false.
	virtual bool SupportsLocalWorkers() { return false; }
	
	virtual bool RequiresRelativePaths() { return false; }

	// Override to return true if this distributed controller supports file virtualization; by doing so the distributed compilation thread will skip writing input files to disk
	// and instead populate FTaskCommandData::InputFileData and FTaskCommandData::SourceData, as well as reading results from memory instead of a file on disk
	virtual bool SupportsVirtualFiles() const { return false; }

	// Sets the maxmium number of local workers. Ignored if this controller does not support local workers.
	virtual void SetMaxLocalWorkers(int32 InMaxNumLocalWorkers) { /*dummy*/ }

	virtual void InitializeController() = 0;
	
	// Returns true if the controller may be used.
	virtual bool IsSupported() = 0;

	// Returns the name of the controller. Used for logging purposes.
	virtual const FString GetName() = 0;

	virtual FString RemapPath(const FString& SourcePath) const { return SourcePath; }

	virtual void Tick(float DeltaSeconds){}

	// Returns a new file path to be used for writing input data to.
	virtual FString CreateUniqueFilePath() = 0;

	// Returns the distributed build statistics since the last call and resets its internal values. Returns false if there are no statistics provided.
	virtual bool PollStats(FDistributedBuildStats& OutStats) { return false; }

	// Launches a task. Returns a future which can be waited on for the results.
	virtual TFuture<FDistributedBuildTaskResult> EnqueueTask(const FTaskCommandData& CommandData) = 0;
	
	static const FName& GetModularFeatureType()
	{
		static FName FeatureTypeName = FName(TEXT("DistributedBuildController"));
		return FeatureTypeName;
	}
};
